{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Game Features Toolset",
	"Description": "Toolset for listing, inspecting, and creating Game Feature Plugins via the AI Toolset Registry.",
	"Category": "Other",
	"EditorOnly": true,
	"IsExperimentalVersion": true,
	"EnabledByDefault": false,
	"Modules": [
		{
			"Name": "GameFeaturesToolset",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit",
			"TargetAllowList": ["Editor"]
		}
	],
	"Plugins": [
		{
			"Name": "ToolsetRegistry",
			"Enabled": true
		},
		{
			"Name": "GameFeatures",
			"Enabled": true
		}
	]
}
