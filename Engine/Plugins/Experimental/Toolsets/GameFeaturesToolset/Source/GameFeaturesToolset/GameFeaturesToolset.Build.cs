// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class GameFeaturesToolset : ModuleRules
{
	public GameFeaturesToolset(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		IWYUSupport = IWYUSupport.Full;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
			);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AssetRegistry",
				"CoreUObject",
				"Engine",
				"GameFeatures",
				"Kismet",
				"PluginUtils",
				"Projects",
				"ToolsetRegistry",
				"UnrealEd",
			}
			);
	}
}
