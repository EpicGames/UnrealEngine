// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFeaturesToolset.h"

#include "GameFeaturesSubsystem.h"
#include "GameFeaturesSubsystemSettings.h"
#include "GameFeatureTypes.h"
#include "Interfaces/IPluginManager.h"
#include "Kismet/KismetSystemLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameFeaturesToolset)

DEFINE_LOG_CATEGORY(LogGameFeaturesToolset);

namespace UE::GameFeaturesToolset::Private
{

	// Maps the engine's ~34 internal GFP states to
	// the simplified user-facing enum. Transitional and
	// error states all collapse to Unknown.
	static EPluginToolsetGFPState ToGFPState(
		EGameFeaturePluginState State)
	{
		switch (State)
		{
		case EGameFeaturePluginState::Uninitialized:
			return EPluginToolsetGFPState::Uninitialized;
		case EGameFeaturePluginState::Installed:
			return EPluginToolsetGFPState::Installed;
		case EGameFeaturePluginState::Registered:
			return EPluginToolsetGFPState::Registered;
		case EGameFeaturePluginState::Loaded:
			return EPluginToolsetGFPState::Loaded;
		case EGameFeaturePluginState::Active:
			return EPluginToolsetGFPState::Active;
		default:
			return EPluginToolsetGFPState::Unknown;
		}
	}

	static void RaiseError(const FString& Message)
	{
		UKismetSystemLibrary::RaiseScriptError(
			FString::Printf(
				TEXT("GameFeaturesToolset: %s"), *Message));
	}

	static UGameFeaturesSubsystem* GetGameFeaturesSubsystem()
	{
		UGameFeaturesSubsystem* Subsystem =	GEngine ?
			GEngine->GetEngineSubsystem<UGameFeaturesSubsystem>() : nullptr;
		if (!Subsystem)
		{
			RaiseError(TEXT("GameFeaturesSubsystem not available. Ensure GameFeatures "
					"plugin is enabled."));
		}
		return Subsystem;
	}


	FString ResolvePluginURLOrRaise(const FString& PluginName,
		UGameFeaturesSubsystem*& OutSubsystem)
	{
		OutSubsystem = GetGameFeaturesSubsystem();
		if (!OutSubsystem)
		{
			return FString();
		}

		FString PluginURL;
		if (!OutSubsystem->GetPluginURLByName(PluginName, PluginURL))
		{
			RaiseError(FString::Printf(TEXT("GFP not found: %s"), *PluginName));
			return FString();
		}

		return PluginURL;
	}
}

TArray<FString> UGameFeaturesToolset::ListEnabledGameFeaturePlugins()
{
	TArray<FString> Names;
	for (const TSharedRef<IPlugin>& Plugin : IPluginManager::Get().GetEnabledPlugins())
	{
		if (UGameFeaturesSubsystem::IsGameFeaturePlugin(Plugin))
		{
			Names.Add(Plugin->GetName());
		}
	}
	Names.Sort();
	return Names;
}

TArray<FString> UGameFeaturesToolset::ListDiscoveredGameFeaturePlugins()
{
	TArray<FString> Names;
	for (const TSharedRef<IPlugin>& Plugin : IPluginManager::Get().GetDiscoveredPlugins())
	{
		if (UGameFeaturesSubsystem::IsGameFeaturePlugin(Plugin))
		{
			Names.Add(Plugin->GetName());
		}
	}
	Names.Sort();
	return Names;
}

bool UGameFeaturesToolset::IsGameFeaturePlugin(const FString& PluginName)
{
	using namespace UE::GameFeaturesToolset::Private;
	TSharedPtr<IPlugin> FoundPlugin = IPluginManager::Get().FindPlugin(PluginName);
	if(!FoundPlugin)
	{
		RaiseError(FString::Printf(TEXT("Could not find a plugin with name '%s'"), *PluginName));
		return false;
	}
	return UGameFeaturesSubsystem::IsGameFeaturePlugin(FoundPlugin);
}

bool UGameFeaturesToolset::IsGameFeatureActive(const FString& PluginName)
{
	using namespace UE::GameFeaturesToolset::Private;

	UGameFeaturesSubsystem* Subsystem = nullptr;
	FString PluginURL = ResolvePluginURLOrRaise(PluginName, Subsystem);
	if (PluginURL.IsEmpty())
	{
		return false;
	}
	check(Subsystem);
	EGameFeaturePluginState State =	Subsystem->GetPluginState(PluginURL);
	return State == EGameFeaturePluginState::Active;
}

EPluginToolsetGFPState
UGameFeaturesToolset::GetGameFeatureState(const FString& PluginName)
{
	using namespace UE::GameFeaturesToolset::Private;

	UGameFeaturesSubsystem* Subsystem = nullptr;
	FString PluginURL = ResolvePluginURLOrRaise(PluginName, Subsystem);
	if (PluginURL.IsEmpty())
	{
		return EPluginToolsetGFPState::Unknown;
	}
	check(Subsystem);

	EGameFeaturePluginState State =	Subsystem->GetPluginState(PluginURL);
	return ToGFPState(State);
}

bool UGameFeaturesToolset::RequestActivateGameFeature(const FString& PluginName)
{
	using namespace UE::GameFeaturesToolset::Private;
	UGameFeaturesSubsystem* Subsystem = nullptr;
	FString PluginURL = ResolvePluginURLOrRaise(PluginName, Subsystem);
	if (PluginURL.IsEmpty())
	{
		return false;
	}
	check(Subsystem);

	Subsystem->LoadAndActivateGameFeaturePlugin(
		PluginURL,
		FGameFeaturePluginLoadComplete::CreateLambda(
			[PluginName](
				const UE::GameFeatures::FResult& Result)
			{
				if (Result.HasError())
				{
					UE_LOG(LogGameFeaturesToolset, Error,
						TEXT("Activate GFP '%s' failed: %s"),
						*PluginName,
						*Result.GetError());
				}
			}));

	return true;
}

bool UGameFeaturesToolset::RequestDeactivateGameFeature(
	const FString& PluginName)
{
	using namespace UE::GameFeaturesToolset::Private;
	UGameFeaturesSubsystem* Subsystem = nullptr;
	FString PluginURL = ResolvePluginURLOrRaise(PluginName, Subsystem);
	if (PluginURL.IsEmpty())
	{
		return false;
	}
	check(Subsystem);

	Subsystem->DeactivateGameFeaturePlugin(
		PluginURL,
		FGameFeaturePluginDeactivateComplete::
		CreateLambda(
			[PluginName](
				const UE::GameFeatures::FResult& Result)
			{
				if (Result.HasError())
				{
					UE_LOG(LogGameFeaturesToolset, Error,
						TEXT("Deactivate GFP '%s' failed: %s"),
						*PluginName,
						*Result.GetError());
				}
			}));

	return true;
}
