// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ToolsetRegistry/ToolsetDefinition.h"
#include "Logging/LogCategory.h"

#include "GameFeaturesToolset.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogGameFeaturesToolset, Log, All);

/// Simplified Game Feature Plugin state for AI tools.
/// Maps the engine's ~34 internal states to user-facing
/// states that an LLM would act on.
UENUM(BlueprintType)
enum class EPluginToolsetGFPState : uint8
{
	Uninitialized,
	Installed,
	Registered,
	Loaded,
	Active,
	Unknown
};

/// Provides tools for listing, activating, and deactivating Game Feature Plugins.
UCLASS(BlueprintType, Hidden)
class UGameFeaturesToolset : public UToolsetDefinition
{
	GENERATED_BODY()

public:

	/**
	 * Lists all enabled Game Feature Plugins  sorted by name. Enabled plugins are the only plugins
	 * known by the Game Features system beyond identifying if a plugin is a Game Feature Plugin.
	 * Use the Plugins toolset to do general plugin enable/disable tasks.
	 * @return Sorted names of all enabled Game Feature Plugins.
	 */
	UFUNCTION(meta = (AICallable), Category = "GameFeatures")
	static TArray<FString> ListEnabledGameFeaturePlugins();

	/**
	 * Lists all discovered Game Feature Plugins sorted by name. This includes enabled and disabled 
	 * plugins. Only enabled plugins are known by the Game Features system beyond identifying if a
	 * plugin is a Game Feature Plugin.
	 * Use the Plugins toolset to do general plugin enable/disable tasks.
	 * @return Sorted names of all discovered Game Feature Plugins.
	 */
	UFUNCTION(meta = (AICallable), Category = "GameFeatures")
	static TArray<FString> ListDiscoveredGameFeaturePlugins();

	/**
	 * Return whether or not a plugin is a Game Feature Plugin. Will error if no plugin of this
	 * name can be found by the Plugin Manager.
	 * @param PluginName Name of the plugin
	 * @return True if the plugin is a Game Feature Plugin
	 */
	UFUNCTION(meta = (AICallable), Category = "GameFeatures")
	static bool IsGameFeaturePlugin(const FString& PluginName);

	/**
	 * Checks whether a Game Feature Plugin is active. Raises an error if the subsystem is unavailable
	 * or the plugin is not found.
	 * Use GetGameFeatureState if you need the current state when the plugin is not active.
	 * @param PluginName Name of the Game Feature Plugin.
	 * @return True if the Game Feature Plugin is active.
	 */
	UFUNCTION(meta = (AICallable), Category = "GameFeatures")
	static bool IsGameFeatureActive(const FString& PluginName);

	/**
	 * Gets the current state of a Game Feature Plugin.
	 * @param PluginName Name of the Game Feature Plugin.
	 * @return Simplified state enum. Raises an error if the subsystem is unavailable or the
	 *         plugin is not found.
	 */
	UFUNCTION(meta = (AICallable), Category = "GameFeatures")
	static EPluginToolsetGFPState GetGameFeatureState(const FString& PluginName);

	/**
	 * Requests activation of a Game Feature Plugin.
	 * Returns true if the activation request was submitted successfully. The actual activation
	 * happens asynchronously -- poll GetGameFeatureState() or IsGameFeatureActive()
	 * to confirm completion. Raises an error if the subsystem is unavailable
	 * or the plugin is not found.
	 * @param PluginName Name of the GFP.
	 * @return True if the request was submitted.
	 */
	UFUNCTION(meta = (AICallable), Category = "GameFeatures")
	static bool RequestActivateGameFeature(const FString& PluginName);

	/**
	 * Requests deactivation of a Game Feature Plugin.
	 * Returns true if the deactivation request was submitted successfully. The actual deactivation
	 * happens asynchronously -- poll GetGameFeatureState() to confirm completion.
	 * Raises an error if the subsystem is unavailable or the plugin is not found.
	 * @param PluginName Name of the GFP.
	 * @return True if the request was submitted.
	 */
	UFUNCTION(meta = (AICallable), Category = "GameFeatures")
	static bool RequestDeactivateGameFeature(const FString& PluginName);

};
