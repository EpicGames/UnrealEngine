// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFeaturesToolset.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "ToolsetRegistry/UToolsetRegistry.h"

class FGameFeaturesToolsetModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		UToolsetRegistry::RegisterToolsetClass(UGameFeaturesToolset::StaticClass());
	}

	virtual void ShutdownModule() override
	{
		UToolsetRegistry::UnregisterToolsetClass(UGameFeaturesToolset::StaticClass());
	}
};

IMPLEMENT_MODULE(FGameFeaturesToolsetModule, GameFeaturesToolset);
