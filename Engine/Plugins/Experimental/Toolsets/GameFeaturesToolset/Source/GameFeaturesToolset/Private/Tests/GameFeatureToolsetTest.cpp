// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetRegistry/AssetRegistryModule.h"
#include "GameFeatureAction.h"
#include "GameFeatureData.h"
#include "GameFeaturesToolset.h"
#include "GameFeaturesSubsystem.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/AutomationTest.h"
#include "ToolsetRegistry/ToolCallExceptionHandler.h"
#include "UObject/StrongObjectPtr.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace
{
	const EAutomationTestFlags TestFlags =
		EAutomationTestFlags::EditorContext |
		EAutomationTestFlags::ProductFilter |
		EAutomationTestFlags::CriticalPriority;
}

BEGIN_DEFINE_SPEC(FGameFeaturesToolsetSpec, "AI.Toolsets.GameFeatureToolset.GameFeaturesToolsetSpec",
	TestFlags)
// Ensure no blueprint script exceptions were caught and clear the handler.
void ExpectNoException()
{
	check(ExceptionHandler.IsValid());
	TestEqual(TEXT("Error"), *ExceptionHandler->GetException(), TEXT(""));
	ExceptionHandler.Reset();
}

// Ensure blueprint script exception was caught and contains the error message, then clear the
// handler.
void ExpectExceptionContains(const FString& ErrorMessage)
{
	check(ExceptionHandler.IsValid());
	TestTrue(
		*FString::Printf(TEXT("Error contains '%s'"), *ErrorMessage),
		ExceptionHandler->GetException().Contains(ErrorMessage));
	ExceptionHandler.Reset();
}

// Ensure blueprint script exception was caught and clear the handler.
void ExpectException()
{
	check(ExceptionHandler.IsValid());
	TestNotEqual(TEXT("Has Error"), *ExceptionHandler->GetException(), TEXT(""));
	ExceptionHandler.Reset();
}
private:
	TUniquePtr<UE::ToolsetRegistry::FToolCallExceptionHandler> ExceptionHandler;

END_DEFINE_SPEC(FGameFeaturesToolsetSpec)

void FGameFeaturesToolsetSpec::Define()
{
	BeforeEach([this]()
	{
		ExceptionHandler = MakeUnique<UE::ToolsetRegistry::FToolCallExceptionHandler>();
	});

	AfterEach([this]()
	{
		if (ExceptionHandler.IsValid())
		{
			ExpectNoException();
		}
	});

	Describe(TEXT("ListEnabledGameFeaturePlugins"), [this]()
	{
		It(TEXT("returns a sorted list"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				const TArray<FString> Names = UGameFeaturesToolset::ListEnabledGameFeaturePlugins();
				for (int32 i = 1; i < Names.Num(); ++i)
				{
					TestTrue(TEXT("Sorted"), Names[i - 1] <= Names[i]);
				}
			});
			ExpectNoException();
		});

		It(TEXT("does not include non-game features plugins"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				const TArray<FString> Names = UGameFeaturesToolset::ListEnabledGameFeaturePlugins();
				// ToolsetRegistry is not a game feature plugin
				TestFalse(TEXT("ToolsetRegistry excluded"), Names.Contains(TEXT("ToolsetRegistry")));
			});
			ExpectNoException();
		});

		It(TEXT("only includes plugins that are enabled by the plugin manager"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				const TArray<FString> Names = UGameFeaturesToolset::ListEnabledGameFeaturePlugins();
				TSet<FString> DiscoveredNames;
				for (const TSharedRef<IPlugin>& Plugin : IPluginManager::Get().GetEnabledPlugins())
				{
					DiscoveredNames.Add(Plugin->GetName());
				}
				for (const FString& Name : Names)
				{
					TestTrue(
						FString::Printf(TEXT("'%s' is a discovered plugin"), *Name),
						DiscoveredNames.Contains(Name));
				}
			});
			ExpectNoException();
		});


		It(TEXT("only includes plugins known to game features subsystem"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				UGameFeaturesSubsystem* Subsystem = GEngine ?
					GEngine->GetEngineSubsystem<UGameFeaturesSubsystem>() : nullptr;
				if (TestNotNull(TEXT("game features subsystem"), Subsystem))
				{
					check(Subsystem); // Silence static analysis warning about Subsystem being null.
					const TArray<FString> Names = UGameFeaturesToolset::ListEnabledGameFeaturePlugins();
					for (const FString& Name : Names)
					{
						FString PluginURL;
						TestTrue(
							FString::Printf(TEXT("GameFeaturesSubsystem GetPluginURLByName succeeded for '%s'"), *Name), 
							Subsystem->GetPluginURLByName(Name, PluginURL));
					}
				}
			});
			ExpectNoException();
		});
	});

	Describe(TEXT("ListDiscoveredGameFeaturePlugins"), [this]()
	{
		It(TEXT("returns a sorted list"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				const TArray<FString> Names = UGameFeaturesToolset::ListDiscoveredGameFeaturePlugins();
				for (int32 i = 1; i < Names.Num(); ++i)
				{
					TestTrue(TEXT("Sorted"), Names[i - 1] <= Names[i]);
				}
			});
			ExpectNoException();
		});

		It(TEXT("does not include non-game features plugins"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				const TArray<FString> Names = UGameFeaturesToolset::ListDiscoveredGameFeaturePlugins();
				// ToolsetRegistry is not a game feature plugin
				TestFalse(TEXT("ToolsetRegistry excluded"), Names.Contains(TEXT("ToolsetRegistry")));
			});
			ExpectNoException();
		});

		It(TEXT("only includes plugins that are discovered by the plugin manager"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				const TArray<FString> Names = UGameFeaturesToolset::ListDiscoveredGameFeaturePlugins();
				TSet<FString> DiscoveredNames;
				for (const TSharedRef<IPlugin>& Plugin : IPluginManager::Get().GetDiscoveredPlugins())
				{
					DiscoveredNames.Add(Plugin->GetName());
				}
				for (const FString& Name : Names)
				{
					TestTrue(
						FString::Printf(TEXT("'%s' is a discovered plugin"), *Name),
						DiscoveredNames.Contains(Name));
				}
			});
			ExpectNoException();
		});
	});

	Describe(TEXT("IsGameFeaturePlugin"), [this]()
	{
		It(TEXT("Nonexistent plugin produces an error"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				TestFalse(TEXT("returned state"),
					UGameFeaturesToolset::IsGameFeaturePlugin(TEXT("UnknownTestPluginXYZ123")));
			});
			ExpectExceptionContains(TEXT("GameFeaturesToolset"));
		});

		It(TEXT("Non game feature plugin returns false"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				TestFalse(TEXT("returned state"),
					UGameFeaturesToolset::IsGameFeaturePlugin(TEXT("ToolsetRegistry")));
			});
			ExpectNoException();
		});

		It(TEXT("active game feature plugin produces true"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				UGameFeaturesSubsystem* Subsystem = GEngine ?
					GEngine->GetEngineSubsystem<UGameFeaturesSubsystem>() : nullptr;

				if (TestNotNull(TEXT("game features subsystem"), Subsystem))
				{
					check(Subsystem); // Silence static analysis warning about Subsystem being null.
					Subsystem->ForEachActiveGameFeature<UGameFeatureData>(
						[this](const UGameFeatureData* Data)
					{
						if (TestNotNull(TEXT("game feature data"), Data))
						{
							FString PluginName;
							Data->GetPluginName(PluginName);
							TestTrue(TEXT("returned state"),
								UGameFeaturesToolset::IsGameFeaturePlugin(PluginName));
						}
					}
					);
				}
			});
			ExpectNoException();
		});
	});

	Describe(TEXT("GetGameFeatureState"), [this]()
	{
		It(TEXT("Nonexistent plugin produces an error"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				TestEqual(TEXT("returned state"), EPluginToolsetGFPState::Unknown, 
					UGameFeaturesToolset::GetGameFeatureState(TEXT("UnknownTestPluginXYZ123")));
			});
			ExpectExceptionContains(TEXT("GameFeaturesToolset"));
		});

		It(TEXT("Non game feature plugin produces an error"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				TestEqual(TEXT("returned state"), EPluginToolsetGFPState::Unknown, 
					UGameFeaturesToolset::GetGameFeatureState(TEXT("ToolsetRegistry")));
			});
			ExpectExceptionContains(TEXT("GameFeaturesToolset"));
		});

		It(TEXT("active game feature plugin produces active result"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				UGameFeaturesSubsystem* Subsystem = GEngine ?
					GEngine->GetEngineSubsystem<UGameFeaturesSubsystem>() : nullptr;

				if (TestNotNull(TEXT("game features subsystem"), Subsystem))
				{
					check(Subsystem); // Silence static analysis warning about Subsystem being null.
					Subsystem->ForEachActiveGameFeature<UGameFeatureData>(
						[this](const UGameFeatureData* Data)
					{
						if (TestNotNull(TEXT("game feature data"), Data))
						{
							FString PluginName;
							Data->GetPluginName(PluginName);
							TestEqual(TEXT("returned state"), EPluginToolsetGFPState::Active,
								UGameFeaturesToolset::GetGameFeatureState(PluginName));
							TestTrue(TEXT("returned active result"),
								UGameFeaturesToolset::IsGameFeatureActive(PluginName));
						}
					}
					);
				}
			});
			ExpectNoException();
		});
	});
}

#endif  // WITH_DEV_AUTOMATION_TESTS
